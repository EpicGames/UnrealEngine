{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Cloth Asset Editor (Deprecated)",
	"Description": "Deprecated plugin, please use the Chaos Cloth Asset Editor Core and Chaos Cloth Asset Usd Dataflow Nodes plugins instead.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [],
	"Plugins": [
		{
			"Name": "ChaosClothAssetEditorCore",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetUsdDataflowNodes",
			"Enabled": true
		}
	]
}
